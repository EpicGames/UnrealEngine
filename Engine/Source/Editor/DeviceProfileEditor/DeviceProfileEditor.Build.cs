// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class DeviceProfileEditor : ModuleRules
{
	public DeviceProfileEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"UnrealEd"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
                "InputCore",
				"Slate",
				"SlateCore",
				"WorkspaceMenuStructure",
				"PropertyEditor",
				"SourceControl",
                "TargetPlatform",
				"SharedSettingsWidgets",
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"MainFrame",
			}
		);
	}
}
