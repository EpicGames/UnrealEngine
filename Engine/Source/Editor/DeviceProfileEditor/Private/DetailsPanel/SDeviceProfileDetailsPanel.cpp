// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsPanel/SDeviceProfileDetailsPanel.h"

#include "Containers/Array.h"
#include "DetailsViewArgs.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "IDetailsView.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/BasicLayoutWidgetSlot.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Misc/CoreMisc.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Types/SlateEnums.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

struct FSlateBrush;


#define LOCTEXT_NAMESPACE "DeviceProfileDetailsPanel"


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SDeviceProfileDetailsPanel::Construct( const FArguments& InArgs )
{
	// Generate our details panel.
	DetailsViewBox = SNew( SVerticalBox );
	RefreshUI();

	ChildSlot
	[
		SNew( SBorder )
		.BorderImage( FAppStyle::GetBrush( "Docking.Tab.ContentAreaBrush" ) )
		[
			SNew( SVerticalBox )
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding( FMargin( 2.0f ) )
			.VAlign( VAlign_Bottom )
			[
				SNew( SHorizontalBox )
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding( 0.0f, 0.0f, 4.0f, 0.0f )
				[
					SNew( SImage )
					.Image( FAppStyle::GetBrush( "LevelEditor.Tabs.Details" ) )
				]
				+ SHorizontalBox::Slot()
					.VAlign( VAlign_Center )
					[
						SNew( STextBlock )
						.Text( LOCTEXT("CVarsLabel", "Console Variables") )
						.TextStyle( FAppStyle::Get(), "Docking.TabFont" )
					]
			]

			+ SVerticalBox::Slot()
			[
				SNew( SHorizontalBox )
				+ SHorizontalBox::Slot()
				[
					DetailsViewBox.ToSharedRef()
				]
			]
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


void SDeviceProfileDetailsPanel::UpdateUIForProfile( const TWeakObjectPtr< UDeviceProfile > InProfile )
{
	ViewingProfile = InProfile;
	RefreshUI();
}


void SDeviceProfileDetailsPanel::RefreshUI()
{
	DetailsViewBox->ClearChildren();

	// initialize settings view
	FDetailsViewArgs DetailsViewArgs;
	{
		DetailsViewArgs.bAllowSearch = true;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.bSearchInitialKeyFocus = true;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.bShowOptions = false;
	}

	SettingsView = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreateDetailView(DetailsViewArgs);

	if( ViewingProfile.IsValid() )
	{
		const FSlateBrush* DeviceProfileTypeIcon = FAppStyle::GetDefaultBrush();
		TArray<ITargetPlatform*> TargetPlatforms = GetTargetPlatformManager()->GetTargetPlatforms();
		if (TargetPlatforms.Num())
		{
			DeviceProfileTypeIcon = FAppStyle::GetBrush(TargetPlatforms[0]->GetPlatformInfo().GetIconStyleName(EPlatformIconSize::Normal));
		}

		SettingsView->SetObject(&*ViewingProfile);
		// If a profile is provided, show the details for this profile.
		DetailsViewBox->AddSlot()
		[
			SNew( SBorder )
			.BorderImage( FAppStyle::GetBrush( "ToolPanel.GroupBorder" ) )
			[
				SNew( SVerticalBox )
				+ SVerticalBox::Slot()
				.HAlign( HAlign_Left )
				.AutoHeight()
				[
					SNew( SHorizontalBox )
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(4.0f, 0.0f, 2.0f, 0.0f)
					[
						SNew(SImage)
						.Image(DeviceProfileTypeIcon)
					]

					+SHorizontalBox::Slot()
					[
						SNew(SVerticalBox)
						+SVerticalBox::Slot()
						.VAlign(VAlign_Center)
						[
							SNew( STextBlock )
							.Text( FText::Format( LOCTEXT( "SelectedDeviceProfileFmt", "{0} selected" ), FText::FromString(ViewingProfile->GetName()) ) )
						]
					]
				]

				/**
				 * CVars part of the details panel
				 */
				+ SVerticalBox::Slot()
				.Padding(4.0f)
				.FillHeight(1.0f)
				[
					SNew(SScrollBox)
					+ SScrollBox::Slot()
					[
						SNew( SBorder )
						.BorderImage( FAppStyle::GetBrush( "Docking.Tab.ContentAreaBrush" ) )
						[
							SNew( SVerticalBox )
							+ SVerticalBox::Slot()
							.FillHeight(1.0f)
							.Padding( FMargin( 4.0f ) )
							[
								SettingsView.ToSharedRef()
							]
						]
					]
				]
			]
		];
	}
	else
	{
		// No profile was selected, so the panel should reflect this
		DetailsViewBox->AddSlot()
			[
				SNew( SBorder )
				.BorderImage( FAppStyle::GetBrush( "ToolPanel.GroupBorder" ) )
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.VAlign( VAlign_Top )
					.HAlign( HAlign_Center )
					.Padding( 4.0f )
					[
						SNew( STextBlock )
						.Text( LOCTEXT("SelectAProfile", "Select a device profile above...") )					
					]

				]
			];
	}
}


#undef LOCTEXT_NAMESPACE
