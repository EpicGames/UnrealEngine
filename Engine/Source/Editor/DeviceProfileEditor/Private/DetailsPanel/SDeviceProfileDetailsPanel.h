// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class IDetailsView;
class SVerticalBox;
class SWidget;
class UDeviceProfile;

#define LOCTEXT_NAMESPACE "DeviceProfileDetailsPanel"


/**
 * Slate widget to display details of a device profile 
 */
class SDeviceProfileDetailsPanel : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS( SDeviceProfileDetailsPanel ){}
		SLATE_DEFAULT_SLOT( FArguments, Content )
	SLATE_END_ARGS()

	/**
	 * Constructs this widget with InArgs
	 */
	void Construct( const FArguments& InArgs );


	/**
	 * Update the UI of the details panel with the provided device profile
	 *
	 * @param In Profile The profile which the details panel should reflect
	 */
	void UpdateUIForProfile( const TWeakObjectPtr< UDeviceProfile > InProfile );

private:

	/** Refresh the UI of the details panel. */
	void RefreshUI();

private:

	/** The profile this panel is showing details for. */
	TWeakObjectPtr< UDeviceProfile > ViewingProfile;

	/** The widget which hosts the details content if a profile is provided. */
	TSharedPtr< SVerticalBox > DetailsViewBox;

	/** Holds the details view. */
	TSharedPtr<IDetailsView> SettingsView;
};


#undef LOCTEXT_NAMESPACE
