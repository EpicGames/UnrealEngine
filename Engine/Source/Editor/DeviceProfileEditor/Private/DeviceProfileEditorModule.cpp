// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeviceProfileEditorModule.h"

#include "Delegates/Delegate.h"
#include "Framework/Docking/TabManager.h"
#include "Internationalization/Internationalization.h"
#include "Modules/ModuleManager.h"
#include "SDeviceProfileEditor.h"
#include "Styling/AppStyle.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#define LOCTEXT_NAMESPACE "DeviceProfileEditor"

IMPLEMENT_MODULE(FDeviceProfileEditorModule, DeviceProfileEditor);

static const FName DeviceProfileEditorName("DeviceProfileEditor");


void FDeviceProfileEditorModule::StartupModule()
{
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(DeviceProfileEditorName, FOnSpawnTab::CreateStatic(&FDeviceProfileEditorModule::SpawnDeviceProfileEditorTab))
		.SetDisplayName( NSLOCTEXT("DeviceProfileEditor", "DeviceProfileEditorTitle", "Device Profiles") )
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "DeviceDetails.Tabs.ProfileEditor"))
		.SetGroup( WorkspaceMenu::GetMenuStructure().GetDeveloperToolsPlatformsCategory() );
}


void FDeviceProfileEditorModule::ShutdownModule()
{
	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(DeviceProfileEditorName);
}


TSharedRef<SDockTab> FDeviceProfileEditorModule::SpawnDeviceProfileEditorTab(const FSpawnTabArgs& SpawnTabArgs)
{
	const TSharedRef<SDockTab> MajorTab = SNew(SDockTab)
		.TabRole(ETabRole::MajorTab);

	const TSharedRef<SDeviceProfileEditor> DeviceProfileEditor = SNew(SDeviceProfileEditor);

	MajorTab->SetContent(DeviceProfileEditor);

	return MajorTab;
}


#undef LOCTEXT_NAMESPACE
