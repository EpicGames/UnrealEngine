// Copyright Epic Games, Inc. All Rights Reserved.

// Module includes
#include "DeviceProfileTextureLODSettingsColumn.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "HAL/PlatformMath.h"
// Property table includes
#include "IPropertyTable.h"
#include "IPropertyTableCell.h"
#include "IPropertyTableCellPresenter.h"
#include "IPropertyTableColumn.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "PropertyHandle.h"
#include "PropertyPath.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakFieldPtr.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SNullWidget.h"

class IPropertyTableUtilities;
class SWidget;
class UObject;

// Misc includes


#define LOCTEXT_NAMESPACE "DeviceProfileEditor"


/**
* Formatter of the Texture LOD Settings property for a device profile.
*/
class FTextureLODSettingsCellPresenter : public TSharedFromThis< FTextureLODSettingsCellPresenter > , public IPropertyTableCellPresenter
{
public:
	/** 
	 * Constructor 
	 */
	FTextureLODSettingsCellPresenter(TWeakObjectPtr<UDeviceProfile> InOwnerProfile, const FOnEditDeviceProfileTextureLODSettingsRequestDelegate& OnTextureLODSettingsEditRequest )
		: OwnerProfile(InOwnerProfile)
		, OnEditTextureLODSettingsRequest(OnTextureLODSettingsEditRequest)
	{
	}

	virtual ~FTextureLODSettingsCellPresenter() {}

	/**
	 * Event handler triggered when the user presses the edit TextureLODSettings button
	 *
	 * @return Whether the event was handled.
	 */
	FReply HandleEditTextureLODSettingsButtonPressed()
	{
		OnEditTextureLODSettingsRequest.ExecuteIfBound(OwnerProfile);
		return FReply::Handled();
	}

public:

	/** Begin IPropertyTableCellPresenter interface */
	virtual TSharedRef<class SWidget> ConstructDisplayWidget() override;

	virtual bool RequiresDropDown() override
	{
		return false;
	}

	virtual TSharedRef< class SWidget > ConstructEditModeCellWidget() override
	{
		return ConstructDisplayWidget();
	}

	virtual TSharedRef< class SWidget > ConstructEditModeDropDownWidget() override
	{
		return SNullWidget::NullWidget;
	}

	virtual TSharedRef< class SWidget > WidgetToFocusOnEdit() override
	{
		return SNullWidget::NullWidget;
	}

	virtual bool HasReadOnlyEditMode() override
	{
		return true;
	}

	virtual FString GetValueAsString() override
	{
		return TEXT("");
	}

	virtual FText GetValueAsText() override
	{
		return FText::FromString(TEXT(""));
	}
	/** End IPropertyTableCellPresenter interface */


private:

	/** The object we will link to */
	TWeakObjectPtr<UDeviceProfile> OwnerProfile;

	/** Delegate triggered when the user opts to edit the TextureLODSettings from the button in this cell */
	FOnEditDeviceProfileTextureLODSettingsRequestDelegate OnEditTextureLODSettingsRequest;
};


TSharedRef<class SWidget> FTextureLODSettingsCellPresenter::ConstructDisplayWidget()
{
	return SNew(SBorder)
	.Padding(0.0f)
	.VAlign(VAlign_Center)
	.HAlign(HAlign_Center)
	.BorderImage(FAppStyle::GetBrush("NoBorder"))
	.Content()
	[
		SNew(SButton)
		.OnClicked(this, &FTextureLODSettingsCellPresenter::HandleEditTextureLODSettingsButtonPressed)
		.ContentPadding(2.0f)
		.ButtonStyle(FAppStyle::Get(), "DeviceDetails.EditButton")
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush("Icons.Edit"))
			.ColorAndOpacity(FSlateColor::UseForeground())
		]
	];
}


FDeviceProfileTextureLODSettingsColumn::FDeviceProfileTextureLODSettingsColumn()
{
}


bool FDeviceProfileTextureLODSettingsColumn::Supports(const TSharedRef< IPropertyTableColumn >& Column, const TSharedRef< IPropertyTableUtilities >& Utilities) const
{
	if( Column->GetDataSource()->IsValid() )
	{
		TSharedPtr< FPropertyPath > PropertyPath = Column->GetDataSource()->AsPropertyPath();
		if( PropertyPath.IsValid() && PropertyPath->GetNumProperties() > 0 )
		{
			const FPropertyInfo& PropertyInfo = PropertyPath->GetRootProperty();
			const FProperty* Property = PropertyInfo.Property.Get();
			if (Property->GetName() == TEXT("TextureLODGroups") && Property->IsA(FArrayProperty::StaticClass()))
			{
				return true;
			}
		}
	}

	return false;
}


TSharedPtr< SWidget > FDeviceProfileTextureLODSettingsColumn::CreateColumnLabel(const TSharedRef< IPropertyTableColumn >& Column, const TSharedRef< IPropertyTableUtilities >& Utilities, const FName& Style) const
{
	return NULL;
}


TSharedPtr< IPropertyTableCellPresenter > FDeviceProfileTextureLODSettingsColumn::CreateCellPresenter(const TSharedRef< IPropertyTableCell >& Cell, const TSharedRef< IPropertyTableUtilities >& Utilities, const FName& Style) const
{
	TSharedPtr< IPropertyHandle > PropertyHandle = Cell->GetPropertyHandle();
	if (PropertyHandle.IsValid())
	{
		TArray<UObject*> OuterObjects;
		PropertyHandle->GetOuterObjects(OuterObjects);
		if (OuterObjects.Num() == 1)
		{
			return MakeShareable(new FTextureLODSettingsCellPresenter(CastChecked<UDeviceProfile>(OuterObjects[0]),OnEditTextureLODSettingsRequestDelegate));
		}
	}

	return NULL;
}


#undef LOCTEXT_NAMESPACE
