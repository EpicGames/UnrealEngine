// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "IPropertyTableCustomColumn.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtrTemplates.h"

class IPropertyTableCell;
class IPropertyTableCellPresenter;
class IPropertyTableColumn;
class IPropertyTableUtilities;
class SWidget;
class UDeviceProfile;

/** Delegate triggered when user opts to edit TextureLODSettings **/
DECLARE_DELEGATE_OneParam(FOnEditDeviceProfileTextureLODSettingsRequestDelegate, const TWeakObjectPtr<UDeviceProfile>&);


/**
 * A property table custom column used to bring the user to an editor which 
 * will manage the Texture LOD Settings associated with the device profile
 */
class FDeviceProfileTextureLODSettingsColumn : public IPropertyTableCustomColumn
{
public:
	FDeviceProfileTextureLODSettingsColumn();

public:

	// IPropertyTableCustomColumn interface

	virtual bool Supports( const TSharedRef< IPropertyTableColumn >& Column, const TSharedRef< IPropertyTableUtilities >& Utilities ) const override;
	virtual TSharedPtr< SWidget > CreateColumnLabel( const TSharedRef< IPropertyTableColumn >& Column, const TSharedRef< IPropertyTableUtilities >& Utilities, const FName& Style ) const override;
	virtual TSharedPtr< IPropertyTableCellPresenter > CreateCellPresenter( const TSharedRef< IPropertyTableCell >& Cell, const TSharedRef< IPropertyTableUtilities >& Utilities, const FName& Style ) const override;

public:

	/**
	 * Delegate used to notify listeners that an edit request was triggered from the property table.
	 *
	 * @return The delegate.
	 */
	FOnEditDeviceProfileTextureLODSettingsRequestDelegate& OnEditTextureLODSettingsRequest()
	{
		return OnEditTextureLODSettingsRequestDelegate;
	}

private:

	/** Delegate triggered when user opts to edit TextureLODSettings. **/
	FOnEditDeviceProfileTextureLODSettingsRequestDelegate OnEditTextureLODSettingsRequestDelegate;
};

