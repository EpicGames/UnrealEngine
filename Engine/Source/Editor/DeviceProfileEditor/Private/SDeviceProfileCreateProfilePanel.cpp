// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDeviceProfileCreateProfilePanel.h"

#include "DeviceProfiles/DeviceProfileManager.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformMath.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Misc/CoreMisc.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "UObject/ObjectPtr.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;


#define LOCTEXT_NAMESPACE "DeviceProfileCreateProfilePanel"


/** Panel layout constants */
namespace DeviceProfileCreateProfileUIConstants
{
	const FMargin ListElementPadding(10.0f, 2.0f);
}


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SDeviceProfileCreateProfilePanel::Construct( const FArguments& InArgs, TWeakObjectPtr< UDeviceProfileManager > InDeviceProfileManager )
{
	DeviceProfileManager = InDeviceProfileManager;

	SelectedDeviceProfileParent = NULL;

	// Create the list of available types we can create profiles for.
	TArray<ITargetPlatform*> TargetPlatforms = GetTargetPlatformManager()->GetTargetPlatforms();
	for (int32 TargetPlatformIndex = 0; TargetPlatformIndex < TargetPlatforms.Num(); ++TargetPlatformIndex)
	{
		TSharedPtr<FString> DeviceProfileType = MakeShareable( new FString( TargetPlatforms[TargetPlatformIndex]->PlatformName() ) );
		DeviceProfileTypes.AddUnique(DeviceProfileType);
	}


	ChildSlot
	[
		// Heading
		SNew( SVerticalBox )
		+SVerticalBox::Slot()
		.Padding( FMargin( 2.0f ) )
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding( 0.0f, 0.0f, 4.0f, 0.0f )
			[
				SNew( SImage )
				.Image( FAppStyle::GetBrush( "LevelEditor.Tabs.Details" ) )
			]
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			[
				SNew(STextBlock)
				.TextStyle( FAppStyle::Get(), "Docking.TabFont" )
				.Text( LOCTEXT("CreateAProfileLabel", "Create A Profile...") )			
			]
		]

		// Content
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew( SBorder )
			.BorderImage( FAppStyle::GetBrush( "ToolPanel.GroupBorder" ) )
			[
				// Name entry
				SNew( SVerticalBox )
				+SVerticalBox::Slot()
				.Padding( FMargin( 4.0f ) )
				.AutoHeight()
				[
					SNew( SVerticalBox )
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew( STextBlock )
						.Font( FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont") ) )
						.Text( LOCTEXT("EnterProfileNameLabel", "Profile Name:") )
					]
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew( SHorizontalBox )
						+SHorizontalBox::Slot()
						.FillWidth(1.0f)
						[
							SAssignNew( DeviceProfileNameTextBox, SEditableTextBox )
							.HintText( LOCTEXT("EnterProfileName", "Enter a new profile name...") )
							.ToolTipText( LOCTEXT("EnterProfileName", "Enter a new profile name...") )
						]
					]
				]

				// Profile Type
				+SVerticalBox::Slot()
				.Padding( FMargin( 4.0f ) )
				.AutoHeight()
				[
					SNew( SVerticalBox )
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew( SHorizontalBox )
						+SHorizontalBox::Slot()
						.HAlign(HAlign_Left)
						.FillWidth(1.0f)
						[
							SNew( STextBlock )
							.Font( FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont") ) )
							.Text( LOCTEXT("EnterProfileTypeLabel", "Profile Type:") )
						]
					]
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew( SHorizontalBox )
						+SHorizontalBox::Slot()
						.FillWidth(1.0f)
						[
							SAssignNew(DeviceProfileTypesComboBox, SComboBox<TSharedPtr<FString>>)
							.OptionsSource( &DeviceProfileTypes )
							.OnGenerateWidget( this, &SDeviceProfileCreateProfilePanel::HandleProfileTypeComboBoxGenarateWidget )
							.OnSelectionChanged(this, &SDeviceProfileCreateProfilePanel::HandleProfileTypeChanged)
							.Content()
							[
								SNew(STextBlock)
								.Text( this, &SDeviceProfileCreateProfilePanel::SetProfileTypeComboBoxContent )
							]
						]
					]
				]

				// Parent
				+SVerticalBox::Slot()
				.Padding( FMargin( 4.0f ) )
				.AutoHeight()
				[
					SNew( SVerticalBox )
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew( SHorizontalBox )
						+SHorizontalBox::Slot()
						.HAlign(HAlign_Left)
						.FillWidth(1.0f)
						[
							SNew( STextBlock )
							.Font( FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont") ) )
							.Text( LOCTEXT("EnterProfileParentLabel", "Select a parent:") )
						]
					]
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew( SHorizontalBox )
						+SHorizontalBox::Slot()
						.FillWidth(1.0f)
						[
							SAssignNew(ParentObjectComboBox, SComboBox<UDeviceProfile*>)
							.OptionsSource( &AvailableBaseObjects )
							.OnGenerateWidget( this, &SDeviceProfileCreateProfilePanel::HandleBaseComboBoxGenerateWidget )
							.IsEnabled( this, &SDeviceProfileCreateProfilePanel::IsBaseProfileComboBoxEnabled )
							.OnSelectionChanged( this, &SDeviceProfileCreateProfilePanel::HandleBaseProfileSelectionChanged )
							.Content()
							[
								SNew(STextBlock)
								.Text( this, &SDeviceProfileCreateProfilePanel::SetBaseProfileComboBoxContent )
							]
						]
					]
				]

				// Create profile button!
				+SVerticalBox::Slot()
				.HAlign(HAlign_Right)
				[
					SNew( SHorizontalBox )
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Center)
					.Padding(4.0f)
					.AutoWidth()
					[
						SNew( SButton )
						.OnClicked( this, &SDeviceProfileCreateProfilePanel::HandleCreateDeviceProfileButtonClicked )
						.IsEnabled( this, &SDeviceProfileCreateProfilePanel::IsCreateProfileButtonEnabled )
						.ToolTipText( LOCTEXT("CreateNewDeviceProfileTooltip", "Create this Device Profile...") )
						.Content()
						[
							SNew( SHorizontalBox )
							+SHorizontalBox::Slot()
							.HAlign(HAlign_Center)
							[
								SNew( STextBlock )
								.Text( LOCTEXT("CreateNewDeviceProfile", "Create Profile") )											
							]
						]
					]
				]
			]
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION



FText SDeviceProfileCreateProfilePanel::SetProfileTypeComboBoxContent() const
{
	return SelectedDeviceProfileType.IsValid() ? FText::FromString(*SelectedDeviceProfileType) : LOCTEXT("SelectType", "Choose a device profile type...");
}


void SDeviceProfileCreateProfilePanel::HandleProfileTypeChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
{
	AvailableBaseObjects.Empty();
	if( NewSelection.IsValid() )
	{
		for (decltype(DeviceProfileManager->Profiles)::TIterator DeviceProfileIt(DeviceProfileManager->Profiles); DeviceProfileIt; ++DeviceProfileIt)
		{
			UDeviceProfile* CurrentProfile = *DeviceProfileIt;
			if( CurrentProfile->DeviceType == *NewSelection )
			{
				AvailableBaseObjects.Add( CurrentProfile );
			}
		}
	}
	ParentObjectComboBox->RefreshOptions();

	SelectedDeviceProfileType = NewSelection;

	SelectedDeviceProfileParent = NULL;
	ParentObjectComboBox->ClearSelection();

}


TSharedRef<SWidget> SDeviceProfileCreateProfilePanel::HandleProfileTypeComboBoxGenarateWidget( TSharedPtr<FString> InItem )
{
	return SNew(SBox)
	.Padding(DeviceProfileCreateProfileUIConstants::ListElementPadding)
	[
		SNew(STextBlock)
		.Text(FText::FromString(*InItem))
	];
}



bool SDeviceProfileCreateProfilePanel::IsBaseProfileComboBoxEnabled() const
{
	return AvailableBaseObjects.Num() > 0;
}


void SDeviceProfileCreateProfilePanel::HandleBaseProfileSelectionChanged( UDeviceProfile* NewSelection, ESelectInfo::Type SelectInfo )
{
	SelectedDeviceProfileParent = NewSelection;
}


FText SDeviceProfileCreateProfilePanel::SetBaseProfileComboBoxContent() const
{
	return SelectedDeviceProfileParent.IsValid() ? FText::FromString(SelectedDeviceProfileParent->GetName()) : LOCTEXT("SelectParent", "Copy properties from...");
}


TSharedRef<SWidget> SDeviceProfileCreateProfilePanel::HandleBaseComboBoxGenerateWidget( UDeviceProfile* InItem )
{
	return SNew(SBox)
	.Padding(DeviceProfileCreateProfileUIConstants::ListElementPadding)
	[
		SNew(STextBlock)
		.Text(FText::FromString(InItem->GetName()))
	];
}


bool SDeviceProfileCreateProfilePanel::IsCreateProfileButtonEnabled() const
{		
	return SelectedDeviceProfileType.IsValid() && !DeviceProfileNameTextBox->GetText().IsEmpty();
}


FReply SDeviceProfileCreateProfilePanel::HandleCreateDeviceProfileButtonClicked()
{
	DeviceProfileManager->CreateProfile( *DeviceProfileNameTextBox->GetText().ToString(), *SelectedDeviceProfileType, SelectedDeviceProfileParent.IsValid() ? SelectedDeviceProfileParent->GetName() : TEXT("") );

	// Set the components for this panel back to default
	ResetComponentsState();

	return FReply::Handled();
}


void SDeviceProfileCreateProfilePanel::ResetComponentsState()
{
	SelectedDeviceProfileType = NULL;
	SelectedDeviceProfileParent = NULL;

	DeviceProfileNameTextBox->SetText( FText::FromString( TEXT( "" ) ) );

	DeviceProfileTypesComboBox->ClearSelection();
	ParentObjectComboBox->ClearSelection();
}


#undef LOCTEXT_NAMESPACE
