// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Misc/Optional.h"
#include "Serialization/Archive.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/SCompoundWidget.h"

class SWidget;

#define LOCTEXT_NAMESPACE "DeviceProfileCreateProfilePanel"

class SCheckBox;
class SEditableTextBox;
class UDeviceProfileManager;

/**
 * Slate widget to allow users to create a new device profile
 */
class SDeviceProfileCreateProfilePanel : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS( SDeviceProfileCreateProfilePanel ){}
		SLATE_DEFAULT_SLOT( FArguments, Content )
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs. */
	void Construct( const FArguments& InArgs, TWeakObjectPtr< UDeviceProfileManager > InDeviceProfileManager );

public:

	/**
	 * Is the create profile button enabled.
	 *
	 * @return True if enabled, false otherwise
	 */
	bool IsCreateProfileButtonEnabled() const;

	/**
	 * Handle create device profile button clicked.
	 *
	 * @return whether the click was handled.
	 */
	FReply HandleCreateDeviceProfileButtonClicked();

public:
	
	/**
	 * Is the select a base profile selection available
	 *
	 * @return true if enabled, false otherwise.
	 */
	bool IsBaseProfileComboBoxEnabled() const;

	/**
	 * Handle base selection changed.
	 *
	 * @param NewSelection The newly selected device profile to use as a parent.
	 * @param SelectionInfo The selection type.
	 */
	void HandleBaseProfileSelectionChanged( UDeviceProfile* NewSelection, ESelectInfo::Type SelectInfo );

	/**
	 * Set the base combo box content.
	 *
	 * @return The device profile parent name.
	 */
	FText SetBaseProfileComboBoxContent() const;

	/**
	 * Handle base combo box generate widget.
	 *
	 * @param InItem The device profile.
	 * @return A text box holding the device profile name.
	 */
	TSharedRef<SWidget> HandleBaseComboBoxGenerateWidget( UDeviceProfile* InItem );

public:
	
	/**
	 * Handle profile type changed.
	 *
	 * @param NewSelection The newly selected device profile type.
	 * @param SelectionInfoThe selection type.
	 */
	void HandleProfileTypeChanged( TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo );
	
	/**
	 * Set the profile type combo box content.
	 *
	 * @return The name of the class the created profile will be.
	 */
	FText SetProfileTypeComboBoxContent() const;

	/**
	 * Handle profile type combo box generate widget.
	 *
	 * @param The type of this in string format.
	 * @return A text box holding the device profile name.
	 */
	TSharedRef<SWidget> HandleProfileTypeComboBoxGenarateWidget( TSharedPtr<FString> InItem );

private:

	/** Reset the components state. */
	void ResetComponentsState();

private:

	/** Holds the Device Profile Class types combo box. */
	TSharedPtr<SComboBox<TSharedPtr<FString>>> DeviceProfileTypesComboBox;
	
	/** The collection of available types you can create a profile for. */
	TArray<TSharedPtr<FString>> DeviceProfileTypes;

	/** Holds the Device Profile Class types combo box. */
	TSharedPtr<SComboBox<UDeviceProfile*>> ParentObjectComboBox;

	/** The collection of device profiles available for you to copy properties from that matches the profile type. */
	TArray<UDeviceProfile*> AvailableBaseObjects;

	/** Holds the device profile combo box. */
	TSharedPtr< SComboBox< UDeviceProfile* > > DeviceProfileBaseComboBox;

	/** Holds the device profile check box. */
	TSharedPtr< SCheckBox > DeviceProfileBaseCheckBox;

	/** Holds the device profile manager. */
	TWeakObjectPtr< UDeviceProfileManager > DeviceProfileManager;

	/** Holds the device profile name text box. */
	TSharedPtr< SEditableTextBox > DeviceProfileNameTextBox;

	/** Holds the device profile name text box. */
	TSharedPtr< SEditableTextBox > DeviceProfileParentNameTextBox;

	/** Holds the selected device profile parent. */
	TWeakObjectPtr< UDeviceProfile > SelectedDeviceProfileParent;

	/** Holds the selected device profile parent. */
	TSharedPtr< FString > SelectedDeviceProfileType;
};


#undef LOCTEXT_NAMESPACE
