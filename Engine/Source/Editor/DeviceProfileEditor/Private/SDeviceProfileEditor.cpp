// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDeviceProfileEditor.h"

#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DeviceProfileConsoleVariableColumn.h"
#include "DeviceProfileTextureLODSettingsColumn.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Docking/WorkspaceItem.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Views/ITypedTableView.h"
#include "HAL/FileManager.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMisc.h"
#include "IPropertyTable.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/BasicLayoutWidgetSlot.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "Misc/Attribute.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SDeviceProfileEditorSingleProfileView.h"
#include "SDeviceProfileSelectionPanel.h"
#include "SSettingsEditorCheckoutNotice.h"
#include "SlotBase.h"
#include "SourceControlOperations.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Textures/SlateIcon.h"
#include "Types/SlateEnums.h"
#include "UObject/Class.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakFieldPtr.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

class FUICommandList;
class IPropertyTableCustomColumn;
class SWidget;
class SWindow;
struct FGeometry;


#define LOCTEXT_NAMESPACE "DeviceProfileEditor"

// Tab names for those available in the Device Profile Editor.
static const FName DeviceProfileEditorTabName("DeviceProfiles");


/** Source control for the default device profile config saves */
class SDeviceProfileSourceControl
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SDeviceProfileSourceControl) {}
		SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_END_ARGS()


	/** Constructs this widget with InArgs. */
	void Construct(const FArguments& InArgs);

	/** Destructor. */
	~SDeviceProfileSourceControl();

	/**
	 * Indicate which SWidgetSwitcher slot should be used to show the user of the source control status
	 */
	int32 HandleNoticeSwitcherWidgetIndex() const
	{
		return bIsDefaultConfigCheckOutNeeded ? 1 : 0;
	}

	/**
	 * Take action to check out the default device profile configuration file when requested.
	 *
	 * @return Whether we handled the event.
	 */
	FReply HandleSaveDefaultsButtonPressed();

	/**
	* Take action to check out the default device profile configuration file when requested.
	*
	* @return Whether we handled the event.
	*/
	FReply HandleCheckoutButtonPressed();


	/**
	 * Check whether the SCC is enabled for the Checkout button to become available.
	 *
	 * @return true if SCC is enabled, false otherwise.
	 */
	bool IsCheckOutAvailable() const;


public:

	// SCompoundWidget interface

	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

private:

	/** Holds the last time checking whether the device profile configuration file needs to be checked out. */
	double LastDefaultConfigCheckOutTime;

	/** Holds a flag indicating whether the section's configuration file needs to be checked out. */
	bool bIsDefaultConfigCheckOutNeeded;

	/** The direct path to all of the default device profile config file. */
	TArray<FString> AbsoluteConfigFilePaths;

	/** True if we've requested a full update for this file */
	TSet<FString> SourceControlUpdated;
};


SDeviceProfileSourceControl::~SDeviceProfileSourceControl()
{

}


FReply SDeviceProfileSourceControl::HandleSaveDefaultsButtonPressed()
{
	UDeviceProfileManager::Get().SaveProfiles(true);

	return FReply::Handled();
}


FReply SDeviceProfileSourceControl::HandleCheckoutButtonPressed()
{
	FText ErrorMessage;
	for (const FString& AbsoluteConfigFilePath : AbsoluteConfigFilePaths)
	{
		if (SettingsHelpers::IsCheckedOut(AbsoluteConfigFilePath, true))
		{
			continue;
		}

		if (!SettingsHelpers::CheckOutOrAddFile(AbsoluteConfigFilePath, false, true, &ErrorMessage))
		{
			if (FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("SaveAsDefaultsSourceControlOperationFailed", "The revision control operation failed. Would you like to make it writable?")) == EAppReturnType::Yes)
			{
				SettingsHelpers::MakeWritable(AbsoluteConfigFilePath, true);
			}
		}
	}

	return FReply::Handled();
}


bool SDeviceProfileSourceControl::IsCheckOutAvailable() const
{
	return ISourceControlModule::Get().IsEnabled() && ISourceControlModule::Get().GetProvider().IsAvailable();
}


void SDeviceProfileSourceControl::Construct(const FArguments& InArgs)
{
	LastDefaultConfigCheckOutTime = 0.0;
	bIsDefaultConfigCheckOutNeeded = true;

	TArray<FString> AllConfigFiles;
	UDeviceProfileManager::Get().GetProfileConfigFiles(AllConfigFiles);

	for (const FString& Path : AllConfigFiles)
	{
		AbsoluteConfigFilePaths.Add(FPaths::ConvertRelativePathToFull(Path));
	}

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("RoundedWarning"))
		.Padding(8.0f)
		[
			SNew(SBox)
			.VAlign(VAlign_Center)
			.Content()
			[
				SNew(SWidgetSwitcher)
				.WidgetIndex(this, &SDeviceProfileSourceControl::HandleNoticeSwitcherWidgetIndex)
				// Unlocked slot
				+ SWidgetSwitcher::Slot()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("Icons.Unlock"))
					]
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(FMargin(8.0f, 0.0f))
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("DeviceProfileEditorSCCUnlockedLabel", "The default device profile configuration is under Revision Control. This file is currently writable."))
					]
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					[
						SNew(SButton)
						.OnClicked(this, &SDeviceProfileSourceControl::HandleSaveDefaultsButtonPressed)
						.Text(LOCTEXT("SaveAsDefaultButtonText", "Save as Default"))
					]
				]
				// Locked slot
				+SWidgetSwitcher::Slot()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("Icons.Lock"))
					]
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(FMargin(8.0f, 0.0f))
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("DeviceProfileEditorSCCLockedLabel", "The default device profile configuration is under Revision Control. This file is currently locked."))
					]
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					[
						SNew(SButton)
						.OnClicked(this, &SDeviceProfileSourceControl::HandleCheckoutButtonPressed)
						.IsEnabled(this, &SDeviceProfileSourceControl::IsCheckOutAvailable)
						.Text(LOCTEXT("CheckOutButtonText", "Check Out File"))
					]
				]
			]
		]
	];
}


void SDeviceProfileSourceControl::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	// cache selected settings object's configuration file state
	if(InCurrentTime - LastDefaultConfigCheckOutTime >= 1.0f)
	{
		bIsDefaultConfigCheckOutNeeded = false;

		for (const FString& AbsoluteConfigFilePath : AbsoluteConfigFilePaths)
		{
			if (FPaths::FileExists(AbsoluteConfigFilePath))
			{
				bool bForceUpdate = !SourceControlUpdated.Contains(AbsoluteConfigFilePath);
				bool bCheckedOut = SettingsHelpers::IsCheckedOut(AbsoluteConfigFilePath, bForceUpdate);
				if (bForceUpdate)
				{
					SourceControlUpdated.Add(AbsoluteConfigFilePath);
				}

				// Handle read only as well as writable files that aren't in source control yet
				if ((ISourceControlModule::Get().IsEnabled() && !bCheckedOut) || IFileManager::Get().IsReadOnly(*AbsoluteConfigFilePath))
				{
					bIsDefaultConfigCheckOutNeeded = true;
					break;
				}
			}
		}
		
		LastDefaultConfigCheckOutTime = InCurrentTime;
	}
}


void SDeviceProfileEditor::Construct( const FArguments& InArgs )
{
	DeviceProfileManager = &UDeviceProfileManager::Get();

	// Setup the tab layout for the editor.
	TSharedRef<FWorkspaceItem> RootMenuGroup = FWorkspaceItem::NewGroup( LOCTEXT("RootMenuGroupName", "Root") );
	DeviceManagerMenuGroup = RootMenuGroup->AddGroup(LOCTEXT("DeviceProfileEditorMenuGroupName", "Device Profile Editor Tabs"));
	{
		TSharedRef<SDockTab> DeviceProfilePropertyEditorTab =
			SNew(SDockTab)
			. TabRole( ETabRole::MajorTab )
			. Label( LOCTEXT("TabTitle", "Device Profile Editor") )
			. ToolTipText( LOCTEXT( "TabTitle_ToolTip", "The Device Profile Editor" ) );

		TabManager = FGlobalTabmanager::Get()->NewTabManager( DeviceProfilePropertyEditorTab );

		TabManager->RegisterTabSpawner( DeviceProfileEditorTabName, FOnSpawnTab::CreateRaw( this, &SDeviceProfileEditor::HandleTabManagerSpawnTab, DeviceProfileEditorTabName) )
			.SetDisplayName(LOCTEXT("DeviceProfilePropertyEditorLabel", "Device Profile Property Editor..."))
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "DeviceDetails.Tabs.ProfileEditor"))
			.SetGroup( DeviceManagerMenuGroup.ToSharedRef() );
	}

	EditorTabStack = FTabManager::NewStack()
		->AddTab(DeviceProfileEditorTabName, ETabState::OpenedTab)
		->SetHideTabWell(true)
		->SetForegroundTab(DeviceProfileEditorTabName);

	// Create the tab layout widget
	const TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout("DeviceProfileEditorLayout_v2.0")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Horizontal)
			->Split
			(
				EditorTabStack.ToSharedRef()
			)
		);


	// Create & initialize main menu
	FMenuBarBuilder MenuBarBuilder = FMenuBarBuilder(TSharedPtr<FUICommandList>());
	MenuBarBuilder.AddPullDownMenu(
		LOCTEXT("WindowMenuLabel", "Window"),
		FText::GetEmpty(),
		FNewMenuDelegate::CreateSP(TabManager.ToSharedRef(), &FTabManager::PopulateTabSpawnerMenu, RootMenuGroup)
	);


	ChildSlot
	[		
		// Create tab well where our property grid etc. will live
		SNew(SSplitter)
		+ SSplitter::Slot()
		.Value(0.3f)
		[
			CreateMainDeviceProfilePanel().ToSharedRef()
		]
		+ SSplitter::Slot()
		.Value(0.7f)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(0.0f, 0.0f, 4.0f, 0.0f))
			[
				SNew(SDeviceProfileSourceControl)
			]
			+SVerticalBox::Slot()
			.Padding(FMargin(0.0f,2.0f))
			.AutoHeight()
			[
				MenuBarBuilder.MakeWidget()
			]
			+ SVerticalBox::Slot()
			[
				TabManager->RestoreFrom(Layout, TSharedPtr<SWindow>()).ToSharedRef()
			]
		]
	];
}


SDeviceProfileEditor::~SDeviceProfileEditor()
{
	if( DeviceProfileManager.IsValid() )
	{
		DeviceProfileManager->SaveProfiles();
	}

	if (TabManager.IsValid())
	{
		TabManager->CloseAllAreas();
	}
}


TSharedPtr< SWidget > SDeviceProfileEditor::CreateMainDeviceProfilePanel()
{
	TSharedPtr< SWidget > PanelWidget = SNew( SSplitter )
		.Orientation( Orient_Vertical )
		+ SSplitter::Slot()
		.Value( 1.0f )
		[
			SNew( SBorder )
			.BorderImage( FAppStyle::GetBrush( "Docking.Tab.ContentAreaBrush" ) )
			[
				SAssignNew( DeviceProfileSelectionPanel, SDeviceProfileSelectionPanel, DeviceProfileManager )
				.OnDeviceProfilePinned( this, &SDeviceProfileEditor::HandleDeviceProfilePinned )
				.OnDeviceProfileUnpinned( this, &SDeviceProfileEditor::HandleDeviceProfileUnpinned )
				.OnDeviceProfileViewAlone( this, &SDeviceProfileEditor::HandleDeviceProfileViewAlone )
			]
		]
		+ SSplitter::Slot()
		.SizeRule(SSplitter::ESizeRule::SizeToContent)
		[
			SNew( SBorder )
			.BorderImage( FAppStyle::GetBrush( "Docking.Tab.ContentAreaBrush" ) )
			[
				SNew( STextBlock )
				.AutoWrapText(true)
				.Text(LOCTEXT("CreateDeviceProfile", "To create a device profile, edit your project's DefaultDeviceProfiles.ini file. Make sure to add to or create a section called [DeviceProfileNameAndTypes]. See Engine/Config/BaseDeviceProfiles.ini for examples."))
			]
		];

	return PanelWidget;
}


TSharedRef<SDockTab> SDeviceProfileEditor::HandleTabManagerSpawnTab( const FSpawnTabArgs& Args, FName TabIdentifier )
{
	TSharedPtr<SWidget> TabWidget = SNullWidget::NullWidget;

	if( TabIdentifier == DeviceProfileEditorTabName )
	{
		TabWidget = SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				[
					// Show the property editor
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.FillWidth(0.375f)
					[
						SNew(SBorder)
						.Padding(2)
						.Content()
						[
							SetupPropertyEditor()
						]
					]
				]

				+ SOverlay::Slot()
				[
					// Conditionally draw a notification that indicates profiles should be pinned to be visible.
					SNew(SVerticalBox)
					.Visibility(this, &SDeviceProfileEditor::GetEmptyDeviceProfileGridNotificationVisibility)
					+ SVerticalBox::Slot()
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.HAlign(HAlign_Center)
							.VAlign(VAlign_Center)
							[
								SNew(SImage)
								.Image(FAppStyle::GetBrush("PropertyEditor.AddColumnOverlay"))
							]

							+ SHorizontalBox::Slot()
							.AutoWidth()
							.HAlign(HAlign_Center)
							.VAlign(VAlign_Center)
							[
								SNew(SImage)
								.Image(FAppStyle::GetBrush("PropertyEditor.RemoveColumn"))
							]

							+ SHorizontalBox::Slot()
							.AutoWidth()
							.HAlign(HAlign_Center)
							.VAlign(VAlign_Center)
							.Padding(FMargin(0, 0, 3, 0))
							[
								SNew(STextBlock)
								.Font(FAppStyle::GetFontStyle("PropertyEditor.AddColumnMessage.Font"))
								.Text(LOCTEXT("GenericPropertiesTitle", "Pin Profiles to Add Columns"))
								.ColorAndOpacity(FAppStyle::GetColor("PropertyEditor.AddColumnMessage.Color"))
							]
						]
					]
				]
			];
	}

	// Return the tab with the relevant widget embedded
	return SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		[
			TabWidget.ToSharedRef()
		];
}


TSharedRef<SDockTab> SDeviceProfileEditor::HandleTabManagerSpawnSingleProfileTab(const FSpawnTabArgs& Args, TWeakObjectPtr< UDeviceProfile > InDeviceProfile)
{
	TSharedPtr<SWidget> TabWidget = SNullWidget::NullWidget;

	TabWidget = SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		[
			SNew(SDeviceProfileEditorSingleProfileView, InDeviceProfile)
		];

	TSharedPtr< SDockTab > Tab = SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		[
			TabWidget.ToSharedRef()
		];

	// Return the tab with the relevant widget embedded
	return Tab.ToSharedRef();
}


void SDeviceProfileEditor::HandleDeviceProfilePinned( const TWeakObjectPtr< UDeviceProfile >& DeviceProfile )
{
	if( !DeviceProfiles.Contains( DeviceProfile.Get() ) )
	{
		DeviceProfiles.Add( DeviceProfile.Get() );
		RebuildPropertyTable();

		TabManager->TryInvokeTab(DeviceProfileEditorTabName);
	}
}


void SDeviceProfileEditor::HandleDeviceProfileUnpinned( const TWeakObjectPtr< UDeviceProfile >& DeviceProfile )
{
	if( DeviceProfiles.Contains( DeviceProfile.Get() ) )
	{
		DeviceProfiles.Remove( DeviceProfile.Get() );
		RebuildPropertyTable();

		TabManager->TryInvokeTab(DeviceProfileEditorTabName);
	}
}


void SDeviceProfileEditor::HandleDeviceProfileViewAlone( const TWeakObjectPtr< UDeviceProfile >& DeviceProfile )
{
	FName TabId = DeviceProfile->GetFName();

	if( !RegisteredTabIds.Contains( TabId ) )
	{
		RegisteredTabIds.Add(TabId);

		TabManager->RegisterTabSpawner(TabId, FOnSpawnTab::CreateRaw(this, &SDeviceProfileEditor::HandleTabManagerSpawnSingleProfileTab, DeviceProfile))
			.SetDisplayName(FText::FromName(TabId))
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "DeviceDetails.Tabs.ProfileEditorSingleProfile"))
			.SetGroup(DeviceManagerMenuGroup.ToSharedRef());
	}

	TabManager->TryInvokeTab(TabId);
}


EVisibility SDeviceProfileEditor::GetEmptyDeviceProfileGridNotificationVisibility() const
{
	// IF we aren't showing any items, our prompt should be visible
	return PropertyTable->GetRows().Num() > 0 ?	EVisibility::Hidden : EVisibility::Visible;
}


TSharedRef< SWidget > SDeviceProfileEditor::SetupPropertyEditor()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>( "PropertyEditor" );

	PropertyTable = PropertyEditorModule.CreatePropertyTable();
	RebuildPropertyTable();

	// Adapt the CVars column as a button to open a single editor which will allow better control of the Console Variables
	TSharedRef<FDeviceProfileConsoleVariableColumn> CVarsColumn = MakeShareable(new FDeviceProfileConsoleVariableColumn());

	// Bind our action to open a single editor when requested from the property table
	CVarsColumn->OnEditCVarsRequest().BindRaw(this, &SDeviceProfileEditor::HandleDeviceProfileViewAlone);

	// Adapt the TextureLODSettings column as a button to open a single editor which will allow better control of the Texture Groups
	TSharedRef<FDeviceProfileTextureLODSettingsColumn> TextureLODSettingsColumn = MakeShareable(new FDeviceProfileTextureLODSettingsColumn());
	TextureLODSettingsColumn->OnEditTextureLODSettingsRequest().BindRaw(this, &SDeviceProfileEditor::HandleDeviceProfileViewAlone);

	// Add our Custom Rows to the table
	TArray<TSharedRef<IPropertyTableCustomColumn>> CustomColumns;
	CustomColumns.Add(CVarsColumn);
	CustomColumns.Add(TextureLODSettingsColumn);

	return PropertyEditorModule.CreatePropertyTableWidget(PropertyTable.ToSharedRef(), CustomColumns);
}


void SDeviceProfileEditor::RebuildPropertyTable()
{
	PropertyTable->SetObjects( DeviceProfiles );
	PropertyTable->SetSelectionMode( ESelectionMode::None );

	PropertyTable->SetIsUserAllowedToChangeRoot( false );

	for (TFieldIterator<FProperty> DeviceProfilePropertyIter( UDeviceProfile::StaticClass() ); DeviceProfilePropertyIter; ++DeviceProfilePropertyIter)
	{
		TWeakFieldPtr< FProperty > DeviceProfileProperty = *DeviceProfilePropertyIter;
		if(DeviceProfileProperty->GetName() != TEXT("Parent") )
		{
			PropertyTable->AddColumn(DeviceProfileProperty);
		}
	}

	PropertyTable->RequestRefresh();
}


#undef LOCTEXT_NAMESPACE
