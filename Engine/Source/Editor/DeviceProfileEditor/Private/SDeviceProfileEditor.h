// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Framework/Docking/TabManager.h"
#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FWorkspaceItem;
class SDockTab;
class SWidget;
class UObject;

#define LOCTEXT_NAMESPACE "DeviceProfileEditor"

class IPropertyTable;
class SDeviceProfileSelectionPanel;
class UDeviceProfile;
class UDeviceProfileManager;

/**
 * Slate widget to allow users to edit Device Profiles
 */
class SDeviceProfileEditor
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS( SDeviceProfileEditor ){}
		SLATE_DEFAULT_SLOT( FArguments, Content )
	SLATE_END_ARGS()


	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs );


	/** Destructor */
	~SDeviceProfileEditor();

public:

	/**
	 * Callback for spawning tabs.
	 *
	 * @param Args The arguments used for the tab that is spawned.
	 * @param TabIdentifier The ID of the tab we want to spawn.
	 */
	TSharedRef<SDockTab> HandleTabManagerSpawnTab( const FSpawnTabArgs& Args, FName TabIdentifier );


	/**
	* Callback for spawning tabs.
	*
	* @param Args The arguments used for the tab that is spawned.
	* @param InDeviceProfile The device profile the single editor is displaying information for.
	*/
	TSharedRef<SDockTab> HandleTabManagerSpawnSingleProfileTab(const FSpawnTabArgs& Args, TWeakObjectPtr<UDeviceProfile> InDeviceProfile);

public:

	/**
	 * Handle a device being pinned to the grid.
	 *
	 * @param DeviceProfile The selected profile.
	 */
	void HandleDeviceProfilePinned( const TWeakObjectPtr< UDeviceProfile >& DeviceProfile );


	/**
	 * Handle the device being unpinned from the grid.
	 *
	 * @param DeviceProfile The deselected profile.
	 */
	void HandleDeviceProfileUnpinned( const TWeakObjectPtr< UDeviceProfile >& DeviceProfile );


	/**
	 * Handle the device being unpinned from the grid.
	 *
	 * @param DeviceProfile The profile that is to be viewed alone.
	 */
	void HandleDeviceProfileViewAlone(const TWeakObjectPtr< UDeviceProfile >& DeviceProfile);


	/**
	 * Get whether the notification which indicates that no profiles currently being viewed in the grid, should be visible.
	 *
	 * @return Whether the Notification is visible or not.
	 */
	EVisibility GetEmptyDeviceProfileGridNotificationVisibility() const;


	/**
	 * Setup the device profile property grid.
	 *
	 * @return The widget containing the property grid.
	 */
	TSharedRef< SWidget > SetupPropertyEditor();

private:

	/**
	 * Create the device profile panel which hosts the functionality to edit device profiles in the editor.
	 *
	 * @return The slate widget which hosts the device profile manipulator.
	 */
	TSharedPtr< SWidget > CreateMainDeviceProfilePanel();
	

	/** Rebuild the device profile property table. */
	void RebuildPropertyTable();

private:

	/** Hold a reference to the device profile manager. */
	TWeakObjectPtr<UDeviceProfileManager> DeviceProfileManager;

	/** The collection of device profile ptrs for the selection process. */
	TArray<UObject*> DeviceProfiles;

	/** Holds the tab manager that manages the front-end's tabs. */
	TSharedPtr<FTabManager> TabManager;

	/** The widget which allows the user to select profiles. */
	TSharedPtr<SDeviceProfileSelectionPanel> DeviceProfileSelectionPanel;

	/** Holds the property table. */
	TSharedPtr<IPropertyTable> PropertyTable;

	/** The list of Single Profile Editor ID's which have already been spawned/registered. */
	TArray<FName> RegisteredTabIds;

	/** Access to the "Window" menu to add our spawned tabs of single profile editors. */
	TSharedPtr<FWorkspaceItem> DeviceManagerMenuGroup;

	/** Holds the tab stack where single profiles and the device profile editor will be spawned to. */
	TSharedPtr<FTabManager::FStack> EditorTabStack;
};


#undef LOCTEXT_NAMESPACE
