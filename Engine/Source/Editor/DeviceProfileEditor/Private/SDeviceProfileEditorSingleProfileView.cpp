// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDeviceProfileEditorSingleProfileView.h"

#include "DetailsViewArgs.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "IDetailsView.h"
#include "Layout/Children.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Styling/AppStyle.h"
#include "Widgets/Layout/SBorder.h"


#define LOCTEXT_NAMESPACE "DeviceProfileEditorSingleProfileView"


void  SDeviceProfileEditorSingleProfileView::Construct(const FArguments& InArgs, TWeakObjectPtr< UDeviceProfile > InDeviceProfileToView)
{
	EditingProfile = InDeviceProfileToView;

	// initialize settings view
	FDetailsViewArgs DetailsViewArgs;
	{
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bSearchInitialKeyFocus = false;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.bShowOptions = false;
	}

	SettingsView = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreateDetailView(DetailsViewArgs);
	SettingsView->SetObject(EditingProfile.Get());

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		[
			SettingsView.ToSharedRef()
		]
	];
}


#undef LOCTEXT_NAMESPACE
