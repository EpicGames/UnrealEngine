// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SWidget;

#define LOCTEXT_NAMESPACE "DeviceProfileEditorSingleProfileView"

class IDetailsView;
class UDeviceProfile;

/**
 * Slate widget to allow users to select device profiles
 */
class  SDeviceProfileEditorSingleProfileView
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SDeviceProfileEditorSingleProfileView) {}
		SLATE_DEFAULT_SLOT( FArguments, Content )
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs, TWeakObjectPtr< UDeviceProfile > InDeviceProfile );

	/** Destructor */
	~SDeviceProfileEditorSingleProfileView()= default;

private:

	/** The profile selected from the current list. */
	TWeakObjectPtr< UDeviceProfile > EditingProfile;

	/** Holds the details view. */
	TSharedPtr<IDetailsView> SettingsView;
};


#undef LOCTEXT_NAMESPACE
