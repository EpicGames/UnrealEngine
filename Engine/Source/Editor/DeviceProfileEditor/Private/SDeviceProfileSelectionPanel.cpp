// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDeviceProfileSelectionPanel.h"

#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Views/ITypedTableView.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/BasicLayoutWidgetSlot.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class ITableRow;
class STableViewBase;
class SWidget;
struct FSlateBrush;


#define LOCTEXT_NAMESPACE "DeviceProfileEditorSelectionPanel"


/**
 * Slate widget for each selection row in the selection list
 */
class SDeviceProfileSelectionRow
	: public SMultiColumnTableRow<TWeakObjectPtr<UDeviceProfile>>
{
public:

	SLATE_BEGIN_ARGS( SDeviceProfileSelectionRow )
		: _OnDeviceProfilePinned()
		, _OnDeviceProfileUnpinned()
		{}
		SLATE_DEFAULT_SLOT( FArguments, Content )

		SLATE_ARGUMENT( TWeakObjectPtr< UDeviceProfile >, SelectedDeviceProfile )
		SLATE_EVENT( FOnDeviceProfilePinned, OnDeviceProfilePinned )
		SLATE_EVENT( FOnDeviceProfileUnpinned, OnDeviceProfileUnpinned )
		SLATE_EVENT( FOnDeviceProfileViewAlone, OnDeviceProfileViewAlone )
	SLATE_END_ARGS()


	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs, const TSharedRef< STableViewBase >& InOwnerTableView );


	/**
	 * Generates the widget for the specified column.
	 *
	 * @param ColumnName The name of the column to generate the widget for.
	 * @return The widget.
	 */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override;

public:

	/**
	 * Handle a state change in the device profile pinning.
	 *
	 * @return Whether the event was handled.
	 */
	FReply HandleDeviceProfilePinStateChanged();


	/**
	* Handle the view single profile button pressed.
	*
	* @return Whether the event was handled.
	*/
	FReply ViewSingleProfile();

	/**
	 * Get the image for the pin of this item.
	 *
	 * @return If the profile is pinned we want to indicate this by providing a different angle for the pin.
	 */
	const FSlateBrush* GetPinnedImage() const;


	/**
	 * Get the display profile name.
	 *
	 * @return The profile display name.
	 */
	FText GetProfileDisplayName() const;

private:

	/** Holds the selected device profile. */
	TWeakObjectPtr< UDeviceProfile > SelectedDeviceProfile;

	/** Delegate executed when a profile is unpinned. */
	FOnDeviceProfileUnpinned OnDeviceProfileUnpinned;

	/** Delegate executed when a profile is pinned. */
	FOnDeviceProfilePinned OnDeviceProfilePinned;

	/** Delegate executed when requesting that a profile be viewed alone. */
	FOnDeviceProfileViewAlone OnDeviceProfileViewAlone;

	/** A reference to the profiles pin button. */
	TSharedPtr< SButton > PinProfileButton;

	/** A reference to the profiles view button. */
	TSharedPtr< SButton > ViewProfileButton;

	/** Whether this profile selection is pinned. */
	bool bIsPinned;
};


/* SDeviceProfileSelectionRow implementation
 *****************************************************************************/

void SDeviceProfileSelectionRow::Construct( const FArguments& InArgs, const TSharedRef< STableViewBase >& InOwnerTableView )
{
	SelectedDeviceProfile = InArgs._SelectedDeviceProfile;
	check( SelectedDeviceProfile.IsValid() && TEXT("An invalid device profile was used for this row.") );
	bIsPinned = SelectedDeviceProfile->bVisible;

	// Delegates for pinning/unpinning a device profile
	OnDeviceProfilePinned = InArgs._OnDeviceProfilePinned;
	OnDeviceProfileUnpinned = InArgs._OnDeviceProfileUnpinned;
	OnDeviceProfileViewAlone = InArgs._OnDeviceProfileViewAlone;

	if( bIsPinned )
	{
		OnDeviceProfilePinned.ExecuteIfBound( SelectedDeviceProfile );
	}

	SMultiColumnTableRow< TWeakObjectPtr< UDeviceProfile > >::Construct(SMultiColumnTableRow< TWeakObjectPtr< UDeviceProfile > >::FArguments().Padding(FMargin(0.f,2.f,0.f,0.f)), InOwnerTableView);
}


TSharedRef< SWidget > SDeviceProfileSelectionRow::GenerateWidgetForColumn( const FName& ColumnName )
{
	TSharedPtr< SWidget > ColumnWidget;

	if( ColumnName == TEXT( "Pin") )
	{
		// Draw a pin to show the state of the profile selection
		ColumnWidget = SAssignNew( PinProfileButton, SButton )
			.IsFocusable( false )
			.ToolTipText(LOCTEXT("PinProfileColumnButtonToolTip", "Pin profile to device profile editor table"))
			.ButtonStyle( FAppStyle::Get(), "NoBorder" )
			.ContentPadding( 0 ) 
			.HAlign( HAlign_Center )
			.VAlign( VAlign_Center )
			.OnClicked( this, &SDeviceProfileSelectionRow::HandleDeviceProfilePinStateChanged )
			[
				SNew( SImage )
				.Image( this, &SDeviceProfileSelectionRow::GetPinnedImage )
			];
	}
	else if( ColumnName == TEXT( "Name") )
	{
		// Show the device profiles name
		ColumnWidget = SNew( STextBlock )
			.Text( this, &SDeviceProfileSelectionRow::GetProfileDisplayName );
	}
	else if (ColumnName == TEXT("View"))
	{
		ColumnWidget = SAssignNew(ViewProfileButton, SButton)
			.IsFocusable(false)
			.ToolTipText(LOCTEXT("ViewSingleProfileColumnButtonToolTip", "View this profile in it's own editor"))
			.ButtonStyle(FAppStyle::Get(), "NoBorder")
			.ContentPadding(0)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.OnClicked(this, &SDeviceProfileSelectionRow::ViewSingleProfile)
			[
				SNew(STextBlock)
				.Text(FText::FromString(TEXT("...")))
				.Font(FAppStyle::GetFontStyle("BoldFont"))
			];
	}

	return ColumnWidget.ToSharedRef();
}


FReply SDeviceProfileSelectionRow::HandleDeviceProfilePinStateChanged()
{
	bIsPinned = !bIsPinned;

	if( bIsPinned )
	{
		OnDeviceProfilePinned.ExecuteIfBound( SelectedDeviceProfile );
	}
	else
	{
		OnDeviceProfileUnpinned.ExecuteIfBound( SelectedDeviceProfile );
	}

	return FReply::Handled();
}


FReply SDeviceProfileSelectionRow::ViewSingleProfile()
{
	OnDeviceProfileViewAlone.ExecuteIfBound( SelectedDeviceProfile );
	return FReply::Handled();
}


const FSlateBrush* SDeviceProfileSelectionRow::GetPinnedImage() const
{
	return bIsPinned ? FAppStyle::GetBrush( "PropertyEditor.RemoveColumn" ) : FAppStyle::GetBrush( "PropertyEditor.AddColumn" );
}


FText SDeviceProfileSelectionRow::GetProfileDisplayName() const
{
	return SelectedDeviceProfile.IsValid() ? FText::FromString(SelectedDeviceProfile->GetName()) : LOCTEXT("InvalidProfile", "Invalid Profile");
}


/* SDeviceProfileSelectionPanel implementation
 *****************************************************************************/

void SDeviceProfileSelectionPanel::Construct( const FArguments& InArgs, TWeakObjectPtr< UDeviceProfileManager > InDeviceProfileManager )
{
	DeviceProfileManager = InDeviceProfileManager;

	// Allocate the delegates for profile selection and profile pinning/unpinning
	OnDeviceProfilePinned = InArgs._OnDeviceProfilePinned;
	OnDeviceProfileUnpinned = InArgs._OnDeviceProfileUnpinned;
	OnDeviceProfileViewAlone = InArgs._OnDeviceProfileViewAlone;

	// Hook up our regen function to keep track of device profile manager changes
	RegenerateProfileListDelegateHandle = DeviceProfileManager->OnManagerUpdated().AddRaw( this, &SDeviceProfileSelectionPanel::RegenerateProfileList );

	ChildSlot
	[
		SNew( SVerticalBox )
		+SVerticalBox::Slot()
		.Padding( FMargin( 2.0f ) )
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding( 0.0f, 0.0f, 4.0f, 0.0f )
			[
				SNew( SImage )
				.Image( FAppStyle::GetBrush( "LevelEditor.Tabs.Details" ) )
			]
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			[
				SNew(STextBlock)
				.Text( LOCTEXT("ExistingProfilesLabel", "Existing Device Profiles...") )
			]
		]
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SScrollBox)
			+ SScrollBox::Slot()
			[
				SNew( SBorder )
				.BorderImage( FAppStyle::GetBrush( "ToolPanel.GroupBorder" ) )
				[
					SAssignNew( ListWidget, SVerticalBox )
				]
			]
		]
	];

	RegenerateProfileList();
}


SDeviceProfileSelectionPanel::~SDeviceProfileSelectionPanel()
{
	if( DeviceProfileManager.IsValid() )
	{
		// Remove the delegate when we are destroyed
		DeviceProfileManager->OnManagerUpdated().Remove( RegenerateProfileListDelegateHandle );
	}
}


TSharedRef< ITableRow > SDeviceProfileSelectionPanel::OnGenerateWidgetForDeviceProfile( TWeakObjectPtr<UDeviceProfile> InItem, const TSharedRef< STableViewBase >& OwnerTable )
{
	// Create the row widget.
	return SNew( SDeviceProfileSelectionRow, OwnerTable )
			.SelectedDeviceProfile( InItem )
			.OnDeviceProfilePinned( OnDeviceProfilePinned )
			.OnDeviceProfileUnpinned( OnDeviceProfileUnpinned )
			.OnDeviceProfileViewAlone( OnDeviceProfileViewAlone );
}


void SDeviceProfileSelectionPanel::RegenerateProfileList()
{
	DeviceProfiles.Empty();
	for( auto It = DeviceProfileManager->Profiles.CreateIterator(); It; It++  )
	{
		DeviceProfiles.Add(*It);
	}

	if( DeviceProfileManager.IsValid() )
	{
		ListWidget->ClearChildren();
		ListWidget->AddSlot()
		.Padding( FMargin(4.0f) )
		[
			// Create a list of device profiles of which we can select to view details and pin to property editor for editing
			SNew( SListView< TWeakObjectPtr<UDeviceProfile> > )
			.ListItemsSource( &DeviceProfiles )
			.SelectionMode( ESelectionMode::Single )
			.OnGenerateRow( this, &SDeviceProfileSelectionPanel::OnGenerateWidgetForDeviceProfile )
			.HeaderRow
			(
				SNew( SHeaderRow )
				+ SHeaderRow::Column( FName( "Pin" ) )
				.FixedWidth( 32.0f )
				[
					// The pin icon doesn't need a title
					SNew( STextBlock )
					.Text( FText::GetEmpty() )
				]
				+ SHeaderRow::Column( FName( "Name" ) )
				.FillWidth( 0.95f )
				[
					SNew( STextBlock )
					.Text( LOCTEXT("NameColumn", "Name" ) )
				]
				+SHeaderRow::Column( FName( "View" ) )
				.FixedWidth( 32.0f )
				[
					// The view icon doesn't need a title
					SNew(STextBlock)
					.Text(FText::GetEmpty())
				]
			)
		];
	}
}


#undef LOCTEXT_NAMESPACE
