// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class ITableRow;
class STableViewBase;
class SVerticalBox;
class SWidget;

#define LOCTEXT_NAMESPACE "DeviceProfileEditorSelectionPanel"

class UDeviceProfile;
class UDeviceProfileManager;

/** Delegate that is executed when a device profile is pinned */
DECLARE_DELEGATE_OneParam(FOnDeviceProfilePinned, const TWeakObjectPtr< UDeviceProfile >&);

/** Delegate that is executed when a device profile is unpinned */
DECLARE_DELEGATE_OneParam(FOnDeviceProfileUnpinned, const TWeakObjectPtr< UDeviceProfile >&);

/** Delegate that is executed when a device profile has been selected to view alone. */
DECLARE_DELEGATE_OneParam( FOnDeviceProfileViewAlone, const TWeakObjectPtr< UDeviceProfile >& );


/**
 * Slate widget to allow users to select device profiles
 */
class SDeviceProfileSelectionPanel
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS( SDeviceProfileSelectionPanel )
		: _OnDeviceProfilePinned()
		, _OnDeviceProfileUnpinned()
		, _OnDeviceProfileViewAlone()
		{}
		SLATE_DEFAULT_SLOT( FArguments, Content )

		SLATE_EVENT( FOnDeviceProfilePinned, OnDeviceProfilePinned )
		SLATE_EVENT( FOnDeviceProfileUnpinned, OnDeviceProfileUnpinned )
		SLATE_EVENT( FOnDeviceProfileViewAlone, OnDeviceProfileViewAlone )
	SLATE_END_ARGS()


	/** Constructs this widget with InArgs. */
	void Construct( const FArguments& InArgs, TWeakObjectPtr< UDeviceProfileManager > InDeviceProfileManager );

	/** Destructor. */
	~SDeviceProfileSelectionPanel();

	/**
	 * Handle generating the device profile widget.
	 *
	 * @param InDeviceProfile The selected device profile.
	 * @param OwnerTable The Owner table.
	 * @return The generated widget.
	 */
	TSharedRef<ITableRow> OnGenerateWidgetForDeviceProfile( TWeakObjectPtr<UDeviceProfile> InDeviceProfile, const TSharedRef< STableViewBase >& OwnerTable );

protected:

	/** Regenerate the list view widget when the device profiles are recreated */
	void RegenerateProfileList();

private:

	/** Holds the device profile manager. */
	TWeakObjectPtr< UDeviceProfileManager > DeviceProfileManager;

	/** The collection of device profiles for the selection process. */
	TArray< TWeakObjectPtr<UDeviceProfile> > DeviceProfiles;

	/** Hold the widget that contains the list view of device profiles. */
	TSharedPtr< SVerticalBox > ListWidget;

	/** Delegate for handling a profile being pinned to the grid. */
	FOnDeviceProfilePinned OnDeviceProfilePinned;

	/** Delegate for handling a profile being unpinned from the grid. */
	FOnDeviceProfileUnpinned OnDeviceProfileUnpinned;

	/** Delegate for handling a request to view the profile in it's own editor. */
	FOnDeviceProfileViewAlone OnDeviceProfileViewAlone;

	/** The profile selected from the current list. */
	TWeakObjectPtr< UDeviceProfile > SelectedProfile;

	/** Handle to the registered RegenerateProfileList delegate */
	FDelegateHandle RegenerateProfileListDelegateHandle;
};


#undef LOCTEXT_NAMESPACE
