// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once


/* Boilerplate
 *****************************************************************************/

#include "Misc/MonolithicHeaderBoilerplate.h"
MONOLITHIC_HEADER_BOILERPLATE()


/* Dependencies
 *****************************************************************************/
#include "Modules/ModuleInterface.h"


/* Private includes
 *****************************************************************************/
#include "DeviceProfileEditorModule.h"
