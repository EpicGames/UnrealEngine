// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "SlateFwd.h"
#include "Templates/SharedPointer.h"

class FSpawnTabArgs;
class SDockTab;

/**
 * Device Profile Editor module
 */
class FDeviceProfileEditorModule : public IModuleInterface
{

public:

	//~ Begin IModuleInterface Interface
	virtual void StartupModule() override;

	virtual void ShutdownModule() override;
	//~ End IModuleInterface Interface

public:
	/**
	 * Create the slate UI for the Device Profile Editor
	 */
	static TSharedRef<SDockTab> SpawnDeviceProfileEditorTab( const FSpawnTabArgs& SpawnTabArgs );
};
